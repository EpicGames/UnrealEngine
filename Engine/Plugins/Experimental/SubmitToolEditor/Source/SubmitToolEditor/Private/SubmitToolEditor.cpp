// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubmitToolEditor.h"

#include "Containers/Ticker.h"
#include "EditorValidatorSubsystem.h"
#include "Internationalization/Regex.h"
#include "ISourceControlWindowsModule.h"
#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"
#include "Misc/Paths.h"
#include "Misc/StringOutputDevice.h"
#include "SourceControlOperations.h"
#include "SubmitToolEditorSettings.h"

#define LOCTEXT_NAMESPACE "FSubmitToolEditor"

DEFINE_LOG_CATEGORY(LogSubmitToolEditor);

FAutoConsoleCommand CVarCommandEnableSubmitTool = FAutoConsoleCommand(
	TEXT("SubmitTool.Enable"),
	TEXT("Enables the submit tool override"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args) {
		USubmitToolEditorSettings* Settings = GetMutableDefault<USubmitToolEditorSettings>();
		
		if (!Args.IsEmpty())
		{
			if (!Args[0].IsEmpty())
			{
				Settings->SubmitToolPath = Args[0];
			}

			if (Args.Num() >= 2 && !Args[1].IsEmpty())
			{
				Settings->SubmitToolArguments = Args[1];
			}

			if (Args.Num() >= 3)
			{
				Settings->bForceSubmitTool = Args[2].ToBool();
			}
		}

		Settings->bSubmitToolEnabled = true;
		if (!GIsBuildMachine && !IsRunningCommandlet())
		{
			FSubmitToolEditorModule::Get().RegisterSubmitOverrideDelegate(Settings);
		}
	})
);


FAutoConsoleCommand CVarCommandDisableSubmitTool = FAutoConsoleCommand(
	TEXT("SubmitTool.Disable"),
	TEXT("Disable the submit tool override"),
	FConsoleCommandDelegate::CreateLambda([]() {
		USubmitToolEditorSettings* Settings = GetMutableDefault<USubmitToolEditorSettings>();
		Settings->bSubmitToolEnabled = false;
		if (!GIsBuildMachine && !IsRunningCommandlet())
		{
			FSubmitToolEditorModule::Get().UnregisterSubmitOverrideDelegate();
		}
		})
);

void FSubmitToolEditorModule::StartupModule()
{
	const USubmitToolEditorSettings* Settings = GetDefault<USubmitToolEditorSettings>();
	if (Settings->bSubmitToolEnabled && !GIsBuildMachine && !IsRunningCommandlet())
	{			
		RegisterSubmitOverrideDelegate(Settings);
	}
}

void FSubmitToolEditorModule::ShutdownModule()
{
	UnregisterSubmitOverrideDelegate();
}

void FSubmitToolEditorModule::RegisterSubmitOverrideDelegate(const USubmitToolEditorSettings* InSettings)
{
	ISourceControlWindowsModule& SourceControlWindowsModule = ISourceControlWindowsModule::Get();
	if (!SourceControlWindowsModule.SubmitOverrideDelegate.IsBound())
	{
		UE_LOGF(LogSubmitToolEditor, Display, "Registering SubmitTool to handle submissions from the editor");

		if (InSettings->SubmitToolPath.IsEmpty())
		{
			UE_LOGF(LogSubmitToolEditor, Error, "Submit Tool path is empty: '%ls'", *InSettings->SubmitToolPath);
			return;
		}

		if (InSettings->SubmitToolArguments.IsEmpty())
		{
			UE_LOGF(LogSubmitToolEditor, Error, "Submit Tool Args are empty: '%ls'", *InSettings->SubmitToolArguments);
			return;
		}

		UE_LOGF(LogSubmitToolEditor, Display, "Registering Submit Tool with Path:'%ls' and Args: '%ls'", *InSettings->SubmitToolPath, *InSettings->SubmitToolArguments);
		ISourceControlWindowsModule::Get().CanSubmitOverrideDelegate.BindRaw(this, &FSubmitToolEditorModule::OnCanSubmitToolOverrideCallback);
		ISourceControlWindowsModule::Get().SubmitOverrideDelegate.BindRaw(this, &FSubmitToolEditorModule::OnSubmitToolOverrideCallback);
	}
}

void FSubmitToolEditorModule::UnregisterSubmitOverrideDelegate()
{
	ISourceControlWindowsModule* SourceControlModule = ISourceControlWindowsModule::TryGet();

	if (SourceControlModule != nullptr && SourceControlModule->SubmitOverrideDelegate.IsBound())
	{
		UE_LOGF(LogSubmitToolEditor, Display, "Unregistering Submit Tool.");
		SourceControlModule->SubmitOverrideDelegate.Unbind();
	}
}

FSubmitOverrideReply FSubmitToolEditorModule::OnCanSubmitToolOverrideCallback(const SSubmitOverrideParameters& InParameters, FText* ErrorMessageOut)
{
	return IsPerforceProvider() ? FSubmitOverrideReply::Handled : FSubmitOverrideReply::ProviderNotSupported;
}

FSubmitOverrideReply FSubmitToolEditorModule::OnSubmitToolOverrideCallback(const SSubmitOverrideParameters& InParameters)
{
	if (!IsPerforceProvider())
	{
		return FSubmitOverrideReply::ProviderNotSupported;
	}

	const USubmitToolEditorSettings* Settings = GetDefault<USubmitToolEditorSettings>();
	FString NormalizedPath(Settings->SubmitToolPath);
	FPaths::MakePlatformFilename(NormalizedPath);

	FString Platform(FGenericPlatformMisc::GetUBTPlatform());

	NormalizedPath = NormalizedPath.Replace(TEXT("$(Platform)"), *Platform);

#if PLATFORM_WINDOWS
	const FString LocalAppData = FPlatformMisc::GetEnvironmentVariable(TEXT("LOCALAPPDATA"));
#elif PLATFORM_MAC
	const FString LocalAppData = FPlatformMisc::GetEnvironmentVariable(TEXT("HOME"));
#elif PLATFORM_LINUX
	const FString LocalAppData = FPaths::Combine(FPlatformMisc::GetEnvironmentVariable(TEXT("HOME")), TEXT(".local"), TEXT("share"));
#else
	static_assert(false);
#endif

	NormalizedPath = NormalizedPath.Replace(TEXT("$(LocalAppData)"), *LocalAppData);

	if (Platform.Equals(TEXT("Win64")))
	{
		if (!NormalizedPath.EndsWith(TEXT(".exe"), ESearchCase::IgnoreCase))
		{
			NormalizedPath += TEXT(".exe");
		}
	}

	if (Platform.Equals(TEXT("Mac")) || Platform.Equals(TEXT("Linux")))
	{
		if (NormalizedPath.EndsWith(TEXT(".exe"), ESearchCase::IgnoreCase))
		{
			NormalizedPath = NormalizedPath.Replace(TEXT(".exe"), TEXT(""), ESearchCase::IgnoreCase);
		}
	}

	if (!FPaths::FileExists(NormalizedPath) && !FPaths::DirectoryExists(NormalizedPath))
	{
		UE_LOGF(LogSubmitToolEditor, Error, "The path is invalid: file does not exist - '%ls'", *NormalizedPath);
		return Settings->bForceSubmitTool ? FSubmitOverrideReply::Error : FSubmitOverrideReply::ProviderNotSupported;
	}

	if (!FPaths::FileExists(NormalizedPath) && !FPaths::DirectoryExists(NormalizedPath))
	{
		UE_LOGF(LogSubmitToolEditor, Error, "The path is invalid: file does not exist - '%ls'", *NormalizedPath);		
		return Settings->bForceSubmitTool ? FSubmitOverrideReply::Error : FSubmitOverrideReply::ProviderNotSupported;
	}

	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();

	if (InParameters.ToSubmit.HasSubtype<FString>())
	{
		return InvokeSubmitTool(SourceControlProvider, NormalizedPath, Settings->SubmitToolArguments, InParameters.Description, InParameters.ToSubmit.GetSubtype<FString>());
	}

	if (InParameters.ToSubmit.HasSubtype<TArray<FString>>())
	{
		return InvokeSubmitTool(SourceControlProvider, NormalizedPath, Settings->SubmitToolArguments, InParameters.Description, InParameters.ToSubmit.GetSubtype<TArray<FString>>());
	}

	UE_LOGF(LogSubmitToolEditor, Error, "The parameter is invalid: it does not contains and identifier nor a list of files");
	return FSubmitOverrideReply::Error;
}

FSubmitOverrideReply FSubmitToolEditorModule::InvokeSubmitTool(ISourceControlProvider& InProvider, const FString& InPath, const FString& InArgs, const FString& InDescription, const FString& InIdentifier)
{
	if (InIdentifier.IsEmpty())
	{
		UE_LOGF(LogSubmitToolEditor, Error, "Identifier was empty.");
		return FSubmitOverrideReply::Error;
	}

	FString Args(InArgs);
	FString Description(InDescription);
	
	const USubmitToolEditorSettings* Settings = GetDefault<USubmitToolEditorSettings>();
	if (Settings->bEnforceDataValidation)
	{
		TArray<FText> ValidationMessages;

		bool bValidChangelist = GetChangelistValidationResult(InProvider, InIdentifier, ValidationMessages);

		Description = UpdateValidationTag(Description, bValidChangelist);

		if (!bValidChangelist)
		{
			SaveChangelistDescription(InProvider, Description, InIdentifier);
			return FSubmitOverrideReply::Handled;
		}

		UpdateOptionalValidationArgs(Settings, ValidationMessages, Args);
		UpdateOptionalValidationTags(Settings, ValidationMessages, Description);
	}

	FString Identifier;
	if (EditChangelistDescription(InProvider, Description, InIdentifier, Identifier))
	{
		return InvokeSubmitTool(InProvider, InPath, Args, Identifier);
	}

	return FSubmitOverrideReply::Error;
}

FSubmitOverrideReply FSubmitToolEditorModule::InvokeSubmitTool(ISourceControlProvider& InProvider, const FString& InPath, const FString& InArgs, const FString& InDescription, const TArray<FString>& InFiles)
{
	FString Identifier;
	if (CreateChangelist(InProvider, InDescription, InFiles, Identifier))
	{
		return InvokeSubmitTool(InProvider, InPath, InArgs, InDescription, Identifier);
	}

	return FSubmitOverrideReply::Error;
}

FSubmitOverrideReply FSubmitToolEditorModule::InvokeSubmitTool(ISourceControlProvider& InProvider, const FString& InPath, const FString& InArgs, const FString& InIdentifier)
{
	if (InIdentifier.IsEmpty())
	{
		return FSubmitOverrideReply::Error;
	}

	FString Port, User, Client, WorkspacePath;
	if (!GetPerforceParameter(InProvider, Port, User, Client, WorkspacePath))
	{
		return FSubmitOverrideReply::Error;
	}

	FString SubstitutedArgs = InArgs
		.Replace(TEXT("$(Port)"), *Port)
		.Replace(TEXT("$(User)"), *User)
		.Replace(TEXT("$(Client)"), *Client)
		.Replace(TEXT("$(Changelist)"), *InIdentifier)
		.Replace(TEXT("$(RootDir)"), *WorkspacePath);

	UE_LOGF(LogSubmitToolEditor, Display, "Invoking submit tool: '%ls %ls'", *InPath, *SubstitutedArgs);
	SubmitToolProcHandle = FPlatformProcess::CreateProc(*InPath, *SubstitutedArgs, true, false, false, nullptr, 0, nullptr, nullptr, nullptr);

	if (!SubmitToolProcHandle.IsValid())
	{
		UE_LOGF(LogSubmitToolEditor, Error, "Submit Tool could not be launched.");
		return FSubmitOverrideReply::Error;
	}

	FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FSubmitToolEditorModule::Tick));

	return FSubmitOverrideReply::Handled;
}

bool FSubmitToolEditorModule::IsPerforceProvider() const
{
	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();

	if (SourceControlProvider.GetName() == TEXT("Perforce"))
	{
		return true;
	}

	UE_LOGF(LogSubmitToolEditor, Warning, "Current Provider is not supported for Submit Tool: '%ls', Using the regular editor source control flow", *SourceControlProvider.GetName().ToString());
	return false;
}

bool FSubmitToolEditorModule::GetPerforceParameter(ISourceControlProvider& InProvider, FString& OutPort, FString& OutUser, FString& OutClient, FString& OutWorkspacePath)
{
	TMap<ISourceControlProvider::EStatus, FString> Status = InProvider.GetStatus();

	FString* Port = Status.Find(ISourceControlProvider::EStatus::Port);
	if (Port == nullptr || Port->IsEmpty())
	{
		UE_LOGF(LogSubmitToolEditor, Error, "Could not get a Port from ISourceControlProvider's status");
		return false;
	}

	FString* User = Status.Find(ISourceControlProvider::EStatus::User);
	if (User == nullptr || User->IsEmpty())
	{
		UE_LOGF(LogSubmitToolEditor, Error, "Could not get a User from ISourceControlProvider's status");
		return false;
	}

	FString* Client = Status.Find(ISourceControlProvider::EStatus::Client);
	if (Client == nullptr || Client->IsEmpty())
	{
		UE_LOGF(LogSubmitToolEditor, Error, "Could not get a Client from ISourceControlProvider's status");
		return false;
	}

	FString* WorkspaceRoot = Status.Find(ISourceControlProvider::EStatus::WorkspacePath);
	if (WorkspaceRoot == nullptr || WorkspaceRoot->IsEmpty())
	{
		UE_LOGF(LogSubmitToolEditor, Error, "Could not get a the Workspace Path ISourceControlProvider's status");
		return false;
	}

	OutPort = *Port;
	OutUser = *User;
	OutClient = *Client;
	OutWorkspacePath = *WorkspaceRoot;

	return true;
}

bool FSubmitToolEditorModule::EditChangelistDescription(ISourceControlProvider& InProvider, const FString& InNewChangelistDescription, const FString& InIdentifier, FString& OutIdentifier)
{
	// default output value is the current identifier
	OutIdentifier = InIdentifier;

	TArray<FSourceControlChangelistRef> Changelists = InProvider.GetChangelists(EStateCacheUsage::Use);
	FSourceControlChangelistStatePtr ChangeListState;
	for (const FSourceControlChangelistRef& Changelist : Changelists)
	{
		if (Changelist->GetIdentifier().Equals(InIdentifier))
		{
			ChangeListState = InProvider.GetState(Changelist, EStateCacheUsage::Use);
			break;
		}
	}

	if (!ChangeListState.IsValid())
	{
		UE_LOGF(LogSubmitToolEditor, Warning, "Could not find changelist '%ls'.", *InIdentifier);
		return false;
	}

	if (ChangeListState->SupportsPersistentDescription())
	{
		TSharedRef<FEditChangelist> EditChangelistOperation = ISourceControlOperation::Create<FEditChangelist>();
		EditChangelistOperation->SetDescription(FText::FromString(InNewChangelistDescription));

		ECommandResult::Type EditCLResult = InProvider.Execute(EditChangelistOperation, ChangeListState->GetChangelist(), EConcurrency::Synchronous);

		if (EditCLResult != ECommandResult::Succeeded)
		{
			UE_LOGF(LogSubmitToolEditor, Warning, "Could not edit changelist '%ls''s description, this is not critical and shall not prevent running the submit tool.", *InIdentifier);
		}

		return true;
	}
	// deal with the default changelist
	else
	{
		TArray<FString> FilesToMove;
		Algo::Transform(ChangeListState->GetFilesStates(), FilesToMove, [](const TSharedRef<ISourceControlState>& FileState) { return FileState->GetFilename(); });

		TSharedRef<FNewChangelist> NewChangelistOperation = ISourceControlOperation::Create<FNewChangelist>();

		NewChangelistOperation->SetDescription(FText::FromString(InNewChangelistDescription));
		ECommandResult::Type NewCLResult = InProvider.Execute(NewChangelistOperation, FilesToMove, EConcurrency::Synchronous);

		if (NewCLResult != ECommandResult::Succeeded)
		{
			UE_LOGF(LogSubmitToolEditor, Warning, "Could not edit changelist '%ls''s description, this is not critical and shall not prevent running the submit tool.", *InIdentifier);
		}

		const FSourceControlChangelistPtr& NewChangeList = NewChangelistOperation->GetNewChangelist();
		OutIdentifier = NewChangeList->GetIdentifier();
		return true;
	}
}

void FSubmitToolEditorModule::SaveChangelistDescription(ISourceControlProvider& InProvider, const FString& InNewChangelistDescription, const FString& InIdentifier)
{
	TArray<FSourceControlChangelistRef> Changelists = InProvider.GetChangelists(EStateCacheUsage::Use);
	FSourceControlChangelistStatePtr ChangeListState;
	for (const FSourceControlChangelistRef& Changelist : Changelists)
	{
		if (Changelist->GetIdentifier().Equals(InIdentifier))
		{
			ChangeListState = InProvider.GetState(Changelist, EStateCacheUsage::Use);
			break;
		}
	}

	if (!ChangeListState.IsValid())
	{
		UE_LOGF(LogSubmitToolEditor, Warning, "Could not find changelist '%ls'.", *InIdentifier);
		return;
	}

	if (ChangeListState->SupportsPersistentDescription())
	{
		TSharedRef<FEditChangelist> EditChangelistOperation = ISourceControlOperation::Create<FEditChangelist>();
		EditChangelistOperation->SetDescription(FText::FromString(InNewChangelistDescription));

		ECommandResult::Type EditCLResult = InProvider.Execute(EditChangelistOperation, ChangeListState->GetChangelist(), EConcurrency::Synchronous);

		if (EditCLResult != ECommandResult::Succeeded)
		{
			UE_LOGF(LogSubmitToolEditor, Warning, "Could not edit changelist '%ls''s description, this is not critical and shall not prevent running the submit tool.", *InIdentifier);
		}
	}
	else
	{
		// do nothing, the description cannot be changed
	}
}

bool FSubmitToolEditorModule::CreateChangelist(ISourceControlProvider & InProvider, const FString & InNewChangelistDescription, const TArray<FString>&InFiles, FString & OutIdentifier)
{
	TSharedRef<FNewChangelist> NewChangelistOperation = ISourceControlOperation::Create<FNewChangelist>();
	NewChangelistOperation->SetDescription(FText::FromString(InNewChangelistDescription));

	ECommandResult::Type CreateCLResult = InProvider.Execute(NewChangelistOperation, InFiles, EConcurrency::Synchronous);

	switch (CreateCLResult)
	{
		case ECommandResult::Type::Succeeded:
		{
			FSourceControlChangelistPtr NewChangeList = NewChangelistOperation->GetNewChangelist();
			OutIdentifier = NewChangeList->GetIdentifier();
			return true;
		}
		case ECommandResult::Type::Failed:
		case ECommandResult::Type::Cancelled:
		default:
			UE_LOGF(LogSubmitToolEditor, Error, "Could not create changelist.");
			break;
	}

	return false;
}

bool FSubmitToolEditorModule::GetChangelistValidationResult(ISourceControlProvider& InProvider, const FString& InIdentifier, TArray<FText>& OutValidationMessages)
{
	if (InIdentifier.IsEmpty())
	{
		return true;
	}

	TArray<FSourceControlChangelistRef> Changelists = InProvider.GetChangelists(EStateCacheUsage::Use);
	TSharedPtr<FSourceControlChangelistPtr> ChangelistToProcess = nullptr;
	for (const FSourceControlChangelistRef& Changelist : Changelists)
	{
		if (Changelist->GetIdentifier().Equals(InIdentifier))
		{
			ChangelistToProcess = MakeShared<FSourceControlChangelistPtr>(Changelist);
			break;
		}
	}
	if (ChangelistToProcess.IsValid())
	{
		FSourceControlPreSubmitChangelistValidationExDelegate ValidationDelegate = ISourceControlModule::Get().GetRegisteredPreSubmitChangelistValidationEx();

		FSourceControlValidationDelegateResult ValidationResult;
		
		ValidationDelegate.ExecuteIfBound(*(ChangelistToProcess), ValidationResult);

		OutValidationMessages = MoveTemp(ValidationResult.AllMessages);
		return (ValidationResult.Result == EDataValidationResult::Valid);
	}
	else
	{
		UE_LOGF(LogSubmitToolEditor, Warning, "Could not find changelist '%ls'.", *InIdentifier);
	}

	return false;
}

FString FSubmitToolEditorModule::UpdateValidationTag(const FString& InDescription, bool bIsvalid)
{
	if (USourceControlPreferences::IsValidationTagEnabled())
	{
		const FText ValidationTag = LOCTEXT("ValidationTag", "#changelist validated");
		const FString ValidationTagStr = ValidationTag.ToString();

		if (!bIsvalid)
		{
			return InDescription.Replace(*ValidationTagStr, TEXT(""), ESearchCase::IgnoreCase);
		}
		else
		{
			if (InDescription.Find(ValidationTagStr, ESearchCase::IgnoreCase) == INDEX_NONE)
			{
				FStringOutputDevice Str;

				Str.SetAutoEmitLineTerminator(true);
				Str.Log(InDescription);
				Str.Log(ValidationTagStr);

				return Str;
			}
		}
	}

	return InDescription;
}

void FSubmitToolEditorModule::UpdateOptionalValidationArgs(USubmitToolEditorSettings const* InSettings, TArrayView<FText> InValidationMessages, FString& InOutArgs)
{
	for (FSubmitToolArgumentTrigger const& Trigger : InSettings->OptionalArgumentTriggers)
	{
		TSharedPtr<FRegexPattern> Pattern = MakeShared<FRegexPattern>(Trigger.RegExMessage, ERegexPatternFlags::CaseInsensitive);

		for (FText const& Message : InValidationMessages)
		{
			FRegexMatcher Regex = FRegexMatcher(*Pattern, Message.ToString());
			if (Regex.FindNext())
			{
				InOutArgs.Appendf(TEXT(" %s"), *Trigger.SubmitToolArgument);
				break;
			}
		}
	}
}

void FSubmitToolEditorModule::UpdateOptionalValidationTags(USubmitToolEditorSettings const* InSettings, TArrayView<FText> InValidationMessages, FString& InOutDescription)
{
	if (InSettings->ChangelistTagTriggers.Num() == 0)
	{
		return;
	}

	TArray<FString> Lines;
	InOutDescription.ParseIntoArrayLines(Lines, /*bCullEmpty*/false);

	for (FSubmitToolChangelistTagTrigger const& Trigger : InSettings->ChangelistTagTriggers)
	{
		// Find matching message.
		TSharedPtr<FRegexPattern> Pattern = MakeShared<FRegexPattern>(Trigger.RegExMessage, ERegexPatternFlags::CaseInsensitive);

		FString TagLine;
		for (int32 MessageIndex = InValidationMessages.Num() - 1; MessageIndex >= 0; --MessageIndex)
		{
			const FString Message = InValidationMessages[MessageIndex].ToString();
			FRegexMatcher Regex = FRegexMatcher(*Pattern, Message);
			if (Regex.FindNext())
			{
				const FString TagValue = Regex.GetCaptureGroup(1);
				if (!TagValue.IsEmpty())
				{
					TagLine = FString::Printf(TEXT("#%s %s"), *Trigger.SubmitToolTag, *TagValue);
					break;
				}
			}
		}

		// Find lines with matching tag.
		const FString TagLineStart = TEXT("#") + Trigger.SubmitToolTag;

		for (int32 LineIndex = Lines.Num() - 1; LineIndex >= 0 ; --LineIndex)
		{
			if (!Lines[LineIndex].StartsWith(TagLineStart, ESearchCase::IgnoreCase))
			{
				continue;
			}

			if (TagLine.IsEmpty())
			{
				// Remove existing tag.
				Lines.RemoveAt(LineIndex);
			}
			else
			{
				// Replace existing tag.
				Lines[LineIndex] = MoveTemp(TagLine);
				// Reset and continue iterating to remove any additional matching tags.
				TagLine.Reset();
			}
		}

		if (!TagLine.IsEmpty())
		{
			// Didn't replace existing tag. So add new tag line.
			Lines.Add(TagLine);
		}
	}

	InOutDescription = FString::Join(Lines, LINE_TERMINATOR);
}

bool FSubmitToolEditorModule::Tick(float InDeltaTime)
{
	if (!FPlatformProcess::IsProcRunning(SubmitToolProcHandle))
	{
		ISourceControlProvider& InProvider = ISourceControlModule::Get().GetProvider();

		// When submit tool is done, trigger a cache refresh for CLs and File status so that the editor can refresh its UI
		TArray<FSourceControlChangelistRef> Cls = InProvider.GetChangelists(EStateCacheUsage::ForceUpdate);

		TSharedRef<FUpdatePendingChangelistsStatus, ESPMode::ThreadSafe> UpdatePendingChangelistsOperation = ISourceControlOperation::Create<FUpdatePendingChangelistsStatus>();
		UpdatePendingChangelistsOperation->SetUpdateAllChangelists(true);
		UpdatePendingChangelistsOperation->SetUpdateFilesStates(true);
		UpdatePendingChangelistsOperation->SetUpdateShelvedFilesStates(true);

		InProvider.Execute(UpdatePendingChangelistsOperation, EConcurrency::Asynchronous);
		return false;
	}

	return true;
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FSubmitToolEditorModule, SubmitToolEditor)