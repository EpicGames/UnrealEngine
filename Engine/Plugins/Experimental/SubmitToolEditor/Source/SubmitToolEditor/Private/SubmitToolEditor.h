// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "HAL/PlatformProcess.h"

DECLARE_LOG_CATEGORY_EXTERN(LogSubmitToolEditor, Log, All);

enum class FSubmitOverrideReply;
class USubmitToolEditorSettings;
class ISourceControlProvider;
struct SSubmitOverrideParameters;

class FSubmitToolEditorModule : public IModuleInterface
{
public:
	/**
	 * Get reference to the SubmitToolEditor module instance
	 */
	static inline FSubmitToolEditorModule& Get()
	{
		return FModuleManager::LoadModuleChecked<FSubmitToolEditorModule>("SubmitToolEditor");
	}

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	void RegisterSubmitOverrideDelegate(const USubmitToolEditorSettings* InSettings);
	void UnregisterSubmitOverrideDelegate();
	
private:

	FSubmitOverrideReply OnCanSubmitToolOverrideCallback(const SSubmitOverrideParameters& InParameters, FText* ErrorMessageOut);
	FSubmitOverrideReply OnSubmitToolOverrideCallback(const SSubmitOverrideParameters& InParameters);
	FSubmitOverrideReply InvokeSubmitTool(ISourceControlProvider& InProvider, const FString& InPath, const FString& InArgs, const FString& InDescription, const FString& InIdentifier);
	FSubmitOverrideReply InvokeSubmitTool(ISourceControlProvider& InProvider, const FString& InPath, const FString& InArgs, const FString& InDescription, const TArray<FString>& InFiles);
	FSubmitOverrideReply InvokeSubmitTool(ISourceControlProvider& InProvider, const FString& InPath, const FString& InArgs, const FString& InIdentifier);

	bool IsPerforceProvider() const;
	bool GetPerforceParameter(ISourceControlProvider& InProvider, FString& OutPort, FString& OutUser, FString& OutClient, FString& OutWorkspacePath);
	bool EditChangelistDescription(ISourceControlProvider& InProvider, const FString& InNewChangelistDescription, const FString& InIdentifier, FString& OutIdentifier);
	void SaveChangelistDescription(ISourceControlProvider& InProvider, const FString& InNewChangelistDescription, const FString& InIdentifier);
	bool CreateChangelist(ISourceControlProvider& InProvider, const FString& InNewChangelistDescription, const TArray<FString>& InFiles, FString& OutIdentifier);
	bool GetChangelistValidationResult(ISourceControlProvider& InProvider, const FString& InIdentifier, TArray<FText>& OutValidationMessages);
	FString UpdateValidationTag(const FString& InDescription, bool bIsvalid);
	void UpdateOptionalValidationArgs(USubmitToolEditorSettings const* InSettings, TArrayView<FText> InValidationMessages, FString& InOutArgs);
	void UpdateOptionalValidationTags(USubmitToolEditorSettings const* InSettings, TArrayView<FText> InValidationMessages, FString& InOutDescription);

	bool Tick(float InDeltaTime);

	FProcHandle SubmitToolProcHandle;
};
