// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubmitToolEditorSettings.h"

#include "SubmitToolEditor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SubmitToolEditorSettings)

USubmitToolEditorSettings::USubmitToolEditorSettings()
{
}

void USubmitToolEditorSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (bSubmitToolEnabled && GIsBuildMachine == false && IsRunningCommandlet() == false)
	{
		FSubmitToolEditorModule::Get().RegisterSubmitOverrideDelegate(this);
	}
	else
	{
		FSubmitToolEditorModule::Get().UnregisterSubmitOverrideDelegate();
	}

	SaveConfig();
}

FName USubmitToolEditorSettings::GetCategoryName() const
{
	return FName("Editor");
}
