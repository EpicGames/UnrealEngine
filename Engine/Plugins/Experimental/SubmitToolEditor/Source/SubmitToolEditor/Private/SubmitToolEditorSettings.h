// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"

#include "SubmitToolEditorSettings.generated.h"

struct FPropertyChangedEvent;

/** Settings for adding SubmitTool changelist tags based on validation messages */
USTRUCT()
struct FSubmitToolChangelistTagTrigger
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = "Submit Tool")
	FString RegExMessage;

	UPROPERTY(EditAnywhere, Category = "Submit Tool")
	FString SubmitToolTag;
};

/** Settings for adding SubmitTool arguments based on validation messages */
USTRUCT()
struct FSubmitToolArgumentTrigger
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = "Submit Tool")
	FString RegExMessage;

	UPROPERTY(EditAnywhere, Category = "Submit Tool")
	FString SubmitToolArgument;
};

/** Settings for the submit tool in the editor */
UCLASS(config=Editor, meta = (DisplayName = "Submit Tool Settings"))
class USubmitToolEditorSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	USubmitToolEditorSettings();
	void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual FName GetCategoryName() const override;

	UPROPERTY(config, Category = "Submit Tool", EditDefaultsOnly)
	FString SubmitToolPath;

	UPROPERTY(config, Category = "Submit Tool", EditDefaultsOnly)
	FString SubmitToolArguments;

	UPROPERTY(config, Category = "Submit Tool", EditDefaultsOnly)
	bool bSubmitToolEnabled;
	
	UPROPERTY(config, Category = "Submit Tool", EditDefaultsOnly)
	bool bForceSubmitTool = true;
	
	UPROPERTY(config, Category = "Submit Tool", EditDefaultsOnly)
	bool bEnforceDataValidation;

	UPROPERTY(config, Category = "Submit Tool", EditDefaultsOnly)
	TArray<FSubmitToolChangelistTagTrigger> ChangelistTagTriggers;

	UPROPERTY(config, Category = "Submit Tool", EditDefaultsOnly)
	TArray<FSubmitToolArgumentTrigger> OptionalArgumentTriggers;
};
