// Copyright Epic Games, Inc. All Rights Reserved.
using UnrealBuildTool;

public class SubmitToolEditor : ModuleRules
{
	public SubmitToolEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"SourceControlWindows",
				"SourceControl",
				"DeveloperSettings",
				"DataValidation"
			}
		);
	}
}