{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Submit Tool Editor Override",
	"Description": "Sets up Submit Tool to be launched by the editor",
	"Category": "Editor",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"EnabledByDefault": false,
	"Modules": [
		{
			"Name": "SubmitToolEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	 "Plugins": [
		{
		"Name": "DataValidation",
		"Enabled": true,
		"Optional": false
		}
	 ]
}